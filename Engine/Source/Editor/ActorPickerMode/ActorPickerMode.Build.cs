// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class ActorPickerMode : ModuleRules
{
    public ActorPickerMode(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
                "InteractiveToolsFramework",
                "Slate",
                "SlateCore",
				"EditorFramework",
				"UnrealEd",
				"EditorInteractiveToolsFramework",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				"LevelEditor",
			}
		);
	}
}
