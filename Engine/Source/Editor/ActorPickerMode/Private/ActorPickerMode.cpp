// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorPickerMode.h"
#include "Modules/ModuleManager.h"
#include "EditorModeRegistry.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "EditorModeActorPicker.h" 
#include "Framework/Application/SlateApplication.h"
#include "LevelEditor.h"

IMPLEMENT_MODULE( FActorPickerModeModule, ActorPickerMode );

void FActorPickerModeModule::StartupModule()
{
	// Ensure the level editor module is loaded for grabbing the mode manager later
	FModuleManager::Get().LoadModuleChecked("LevelEditor");
	FEditorModeRegistry::Get().RegisterMode<FEdModeActorPicker>(FBuiltinEditorModes::EM_ActorPicker);

	if (FSlateApplication::IsInitialized())
	{
		OnApplicationDeactivatedHandle = FSlateApplication::Get().OnApplicationActivationStateChanged().Add(TDelegate<void(const bool)>::CreateRaw(this, &FActorPickerModeModule::OnApplicationDeactivated));
	}
}

void FActorPickerModeModule::ShutdownModule()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().OnApplicationActivationStateChanged().Remove(OnApplicationDeactivatedHandle);
		OnApplicationDeactivatedHandle.Reset();
	}

	FEditorModeRegistry::Get().UnregisterMode(FBuiltinEditorModes::EM_ActorPicker);
}

void FActorPickerModeModule::BeginActorPickingMode(FOnGetAllowedClasses InOnGetAllowedClasses, FOnShouldFilterActor InOnShouldFilterActor, FOnActorSelected InOnActorSelected) const
{
	if (FEditorModeTools* ModeTools = GetLevelEditorModeManager())
	{
		// Activate the mode
		ModeTools->ActivateMode(FBuiltinEditorModes::EM_ActorPicker);

		// Set the required delegates
		FEdModeActorPicker* Mode = ModeTools->GetActiveModeTyped<FEdModeActorPicker>(FBuiltinEditorModes::EM_ActorPicker);
		if (ensure(Mode))
		{
			Mode->OnActorSelected = InOnActorSelected;
			Mode->OnGetAllowedClasses = InOnGetAllowedClasses;
			Mode->OnShouldFilterActor = InOnShouldFilterActor;
		}
	}
}

void FActorPickerModeModule::OnApplicationDeactivated(const bool IsActive) const
{
	if (!IsActive) 
	{ 
		EndActorPickingMode();
	}
}

void FActorPickerModeModule::EndActorPickingMode() const
{
	if (FEditorModeTools* ModeTools = GetLevelEditorModeManager())
	{
		ModeTools->DeactivateMode(FBuiltinEditorModes::EM_ActorPicker);
	}
}

bool FActorPickerModeModule::IsInActorPickingMode() const
{
	if (FEditorModeTools* ModeTools = GetLevelEditorModeManager())
	{
		return ModeTools->IsModeActive(FBuiltinEditorModes::EM_ActorPicker);
	}
	return false;
}

FEditorModeTools* FActorPickerModeModule::GetLevelEditorModeManager()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> FirstLevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (FirstLevelEditor.IsValid())
	{
		return &FirstLevelEditor->GetEditorModeManager();
	}

	return nullptr;
}
