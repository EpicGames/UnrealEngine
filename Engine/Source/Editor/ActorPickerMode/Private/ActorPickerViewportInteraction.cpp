// Copyright Epic Games, Inc. All Rights Reserved.


#include "ActorPickerViewportInteraction.h"

#include "EditorModeActorPicker.h"
#include "BaseBehaviors/MouseHoverBehavior.h"

UActorPickerViewportInteraction::UActorPickerViewportInteraction()
{
	InteractionName = TEXT("Actor Picker Interaction");
	Groups = { UE::Editor::ViewportInteractions::ViewportClick };
}

void UActorPickerViewportInteraction::BuildBehaviors()
{
	Super::BuildBehaviors();
	
	using namespace UE::Editor::ViewportInteractions;
	
	if (UViewportClickBehavior* ClickBehavior = ClickBehaviorWeak.Get())
	{
		ClickBehavior->SetBindings({
			EKeys::LeftMouseButton
		});
		
		// Beat the normal click, but still work with drags.
		ClickBehavior->SetDefaultPriority(FInputCapturePriority(CLICK_PRIORITY).MakeHigher(VIEWPORT_INTERACTION_PARTIAL_STEP));
	}
	
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	HoverBehavior->SetDefaultPriority(FInputCapturePriority::DEFAULT_TOOL_PRIORITY);
	RegisterInputBehavior(HoverBehavior);
}

void UActorPickerViewportInteraction::OnClickDown(const FInputDeviceRay& InClickPos)
{
	if (!OnUpdateHover(InClickPos))
	{
		SetMouseCursorOverride(EMouseCursor::Default);
	}
}

FInputRayHit UActorPickerViewportInteraction::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	if (GetHitProxy(PressPos))
	{
		return FInputRayHit(0.0);
	}
	return FInputRayHit();
}

void UActorPickerViewportInteraction::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	OnUpdateHover(DevicePos);
}

bool UActorPickerViewportInteraction::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	if (const TSharedPtr<FEdModeActorPicker> Mode = WeakMode.Pin())
	{
		Mode->UpdateHoveredActor(GetHitProxy(DevicePos));
		SetMouseCursorOverride(Mode->PickState == EPickState::OverActor ? EMouseCursor::EyeDropper : EMouseCursor::SlashedCircle);
		return true;
	}
	return false;
}

void UActorPickerViewportInteraction::OnEndHover()
{
	ClearMouseCursorOverride();
}

void UActorPickerViewportInteraction::SetMode(const TSharedPtr<FEdModeActorPicker>& InMode)
{
	WeakMode = InMode;
}

void UActorPickerViewportInteraction::ProcessClick_Internal(const FViewportClick& InViewportClick, HHitProxy* InHitProxy, FSceneView& View)
{
	if (const TSharedPtr<FEdModeActorPicker> Mode = WeakMode.Pin())
	{
		ClearMouseCursorOverride();
		Mode->OnTrySelectActor(InHitProxy);
	}
}
