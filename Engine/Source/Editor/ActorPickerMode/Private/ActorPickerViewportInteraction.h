// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ViewportInteractions/ViewportClickInteraction.h"
#include "ActorPickerViewportInteraction.generated.h"

class FEdModeActorPicker;

UCLASS()
class UActorPickerViewportInteraction : public UViewportClickInteraction, public IHoverBehaviorTarget
{
	GENERATED_BODY()
	
public:
	UActorPickerViewportInteraction();
	
	virtual void OnClickDown(const FInputDeviceRay& InClickPos) override;

	virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual void OnEndHover() override;
	
	void SetMode(const TSharedPtr<FEdModeActorPicker>& InMode);

protected:
	virtual void BuildBehaviors() override;
	virtual void ProcessClick_Internal(const FViewportClick& InViewportClick, HHitProxy* InHitProxy, FSceneView& View) override;
	
	TWeakPtr<FEdModeActorPicker> WeakMode;
};
