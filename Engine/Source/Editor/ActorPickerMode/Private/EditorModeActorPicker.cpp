// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorModeActorPicker.h"

#include "ActorPickerViewportInteraction.h"
#include "EditorModeManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SToolTip.h"
#include "EngineUtils.h"
#include "LevelEditorViewport.h"
#include "EditorModes.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "PropertyPicker"

FEdModeActorPicker::FEdModeActorPicker()
{
}

void FEdModeActorPicker::Enter()
{
	FEdMode::Enter();
	PickState = EPickState::NotOverViewport;
	HoveredActor.Reset();
	CursorDecoratorWindow = SWindow::MakeCursorDecorator();
	FSlateApplication::Get().AddWindow(CursorDecoratorWindow.ToSharedRef(), true);
	CursorDecoratorWindow->SetContent(
		SNew(SToolTip)
		.Text(this, &FEdModeActorPicker::GetCursorDecoratorText)
		);
		
	GetModeManager()->GetInteractiveToolsContext()->OnBuildViewportInteractions().AddSP(this, &FEdModeActorPicker::OnBuildViewportInteractions);
}

void FEdModeActorPicker::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	if(CursorDecoratorWindow.IsValid())
	{
		CursorDecoratorWindow->MoveWindowTo(FSlateApplication::Get().GetCursorPos() + FSlateApplication::Get().GetCursorSize());
	}

	FEdMode::Tick(ViewportClient, DeltaTime);
}

bool FEdModeActorPicker::MouseEnter(FEditorViewportClient* ViewportClient, FViewport* Viewport,int32 x, int32 y)
{
	PickState = EPickState::OverViewport;
	HoveredActor.Reset();
	UpdateWidgetVisibility(WidgetVisibilityState::StoreAndHide, ViewportClient);
	return FEdMode::MouseEnter(ViewportClient, Viewport, x, y);
}

bool FEdModeActorPicker::MouseLeave(FEditorViewportClient* ViewportClient, FViewport* Viewport)
{
	PickState = EPickState::NotOverViewport;
	HoveredActor.Reset();
	UpdateWidgetVisibility(WidgetVisibilityState::Restore);
	return FEdMode::MouseLeave(ViewportClient, Viewport);
}

bool FEdModeActorPicker::MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y)
{
	if (ViewportInteraction.IsValid())
	{
		return false;
	}

	if (ViewportClient == GCurrentLevelEditingViewportClient)
	{
		int32 HitX = Viewport->GetMouseX();
		int32 HitY = Viewport->GetMouseY();
		UpdateHoveredActor(Viewport->GetHitProxy(HitX, HitY));
	}
	else
	{
		PickState = EPickState::NotOverViewport;
		HoveredActor.Reset();
	}

	return true;
}

bool FEdModeActorPicker::LostFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport)
{
	if (ViewportClient == GCurrentLevelEditingViewportClient)
	{
		UpdateWidgetVisibility(WidgetVisibilityState::Restore);
		// Make sure actor picking mode is disabled once the active viewport loses focus
		RequestDeletion();
		return true;
	}

	return false;
}

bool FEdModeActorPicker::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	if (ViewportClient == GCurrentLevelEditingViewportClient)
	{
		if (Key == EKeys::LeftMouseButton && Event == IE_Pressed && !ViewportInteraction.IsValid())
		{
			// See if we clicked on an actor
			int32 HitX = Viewport->GetMouseX();
			int32 HitY = Viewport->GetMouseY();
			OnTrySelectActor(Viewport->GetHitProxy(HitX, HitY));
			return true;
		}
		else if(Key == EKeys::Escape && Event == IE_Pressed)
		{
			UpdateWidgetVisibility(WidgetVisibilityState::Restore);
			RequestDeletion();
			return true;
		}
	}
	else
	{
		UpdateWidgetVisibility(WidgetVisibilityState::Restore);
		RequestDeletion();
	}

	return false;
}

bool FEdModeActorPicker::GetCursor(EMouseCursor::Type& OutCursor) const
{
	if(HoveredActor.IsValid() && PickState == EPickState::OverActor)
	{
		OutCursor = EMouseCursor::EyeDropper;
	}
	else
	{
		OutCursor = EMouseCursor::SlashedCircle;
	}
	
	return true;
}

bool FEdModeActorPicker::UsesToolkits() const
{
	return false;
}

bool FEdModeActorPicker::IsCompatibleWith(FEditorModeID OtherModeID) const
{
	// We want to be able to perform this action with all the built-in editor modes
	return OtherModeID != FBuiltinEditorModes::EM_None;
}

void FEdModeActorPicker::Exit()
{
	OnActorSelected = FOnActorSelected();
	OnGetAllowedClasses = FOnGetAllowedClasses();
	OnShouldFilterActor = FOnShouldFilterActor();

	if (CursorDecoratorWindow.IsValid())
	{
		CursorDecoratorWindow->RequestDestroyWindow();
		CursorDecoratorWindow.Reset();
	}

	HoveredActor.Reset();
	PickState = EPickState::NotOverViewport;

	UpdateWidgetVisibility(WidgetVisibilityState::Restore);
	
	ViewportInteraction.Reset();
	GetModeManager()->GetInteractiveToolsContext()->OnBuildViewportInteractions().RemoveAll(this);
	
	FEdMode::Exit();
}

FText FEdModeActorPicker::GetCursorDecoratorText() const
{
	switch(PickState)
	{
	default:
	case EPickState::NotOverViewport:
		return LOCTEXT("PickActor_NotOverViewport", "Pick an actor by clicking on it in the active level viewport");
	case EPickState::OverViewport:
		return LOCTEXT("PickActor_NotOverActor", "Pick an actor by clicking on it");
	case EPickState::OverIncompatibleActor:
		{
			if(HoveredActor.IsValid())
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Actor"), FText::FromString(HoveredActor.Get()->GetActorNameOrLabel()));
				return FText::Format(LOCTEXT("PickActor_OverIncompatibleActor", "{Actor} is incompatible"), Arguments);
			}
			else
			{
				return LOCTEXT("PickActor_NotOverActor", "Pick an actor by clicking on it");
			}
		}
	case EPickState::OverActor:
		{
			if(HoveredActor.IsValid())
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Actor"), FText::FromString(HoveredActor.Get()->GetActorNameOrLabel()));
				return FText::Format(LOCTEXT("PickActor_OverActor", "Pick {Actor}"), Arguments);
			}
			else
			{
				return LOCTEXT("PickActor_NotOverActor", "Pick an actor by clicking on it");
			}
		}
	}
}

void FEdModeActorPicker::UpdateHoveredActor(const HHitProxy* HitProxy)
{
	if (HitProxy && HitProxy->IsA(HActor::StaticGetType()))
	{
		const HActor* ActorHit = static_cast<const HActor*>(HitProxy);
		if (ActorHit->Actor)
		{
			AActor* Actor = ActorHit->Actor;
			if (Actor->IsSelectionChild())
			{
				Actor = Actor->GetRootSelectionParent();
			}
			HoveredActor = Actor;
			PickState = IsActorValid(Actor) ? EPickState::OverActor : EPickState::OverIncompatibleActor;
			
			return;
		}
	}
	
	HoveredActor.Reset();
	PickState = EPickState::OverViewport;
}

void FEdModeActorPicker::OnTrySelectActor(const HHitProxy* HitProxy)
{
	UpdateHoveredActor(HitProxy);
	if (HoveredActor.IsValid() && PickState == EPickState::OverActor)
	{
		OnActorSelected.ExecuteIfBound(HoveredActor.Get());
		UpdateWidgetVisibility(WidgetVisibilityState::Restore);
		RequestDeletion();
	}
}

bool FEdModeActorPicker::IsActorValid(const AActor *const Actor) const
{
	bool bIsValid = false;

	if(Actor)
	{
		bool bHasValidClass = true;
		if(OnGetAllowedClasses.IsBound())
		{
			bHasValidClass = false;

			TArray<const UClass*> AllowedClasses;
			OnGetAllowedClasses.Execute(AllowedClasses);
			for(const UClass* AllowedClass : AllowedClasses)
			{
				if ((AllowedClass->IsChildOf(UInterface::StaticClass()) && Actor->GetClass()->ImplementsInterface(AllowedClass)) ||
					Actor->IsA(AllowedClass))
				{
					bHasValidClass = true;
					break;
				}
			}
		}

		bool bHasValidActor = true;
		if(OnShouldFilterActor.IsBound())
		{
			bHasValidActor = OnShouldFilterActor.Execute(Actor);
		}

		bIsValid = bHasValidClass && bHasValidActor;
	}

	return bIsValid;
}

void FEdModeActorPicker::OnBuildViewportInteractions(UViewportInteractionsBehaviorSource* Source)
{
	UE_LOGF(LogTemp, Log, "Building Actor Picker interactions")
	if (UActorPickerViewportInteraction* Interaction = Cast<UActorPickerViewportInteraction>(Source->AddInteraction(UActorPickerViewportInteraction::StaticClass())))
	{
		ViewportInteraction = Interaction;
		Interaction->SetMode(SharedThis(this));
	}
}

void FEdModeActorPicker::UpdateWidgetVisibility(const WidgetVisibilityState InState, FEditorViewportClient* InViewportClient)
{
	if (WidgetVisibilityFunction)
	{
		// restore any stored flags
		WidgetVisibilityFunction();
		WidgetVisibilityFunction.Reset();
	}
	
	if (InState == WidgetVisibilityState::StoreAndHide && InViewportClient)
	{
		// store ModeWidgets flag
		const bool bPreviousModeWidgets = InViewportClient->EngineShowFlags.ModeWidgets;
		WidgetVisibilityFunction = [InViewportClient, bPreviousModeWidgets]()
		{
			if (InViewportClient)
			{
				InViewportClient->EngineShowFlags.SetModeWidgets(bPreviousModeWidgets);		
				InViewportClient->Invalidate(false, false);
			}
		};

		// disable ModeWidgets flag in that vpc
		InViewportClient->EngineShowFlags.SetModeWidgets(false);
		InViewportClient->Invalidate(false, false);
	}
}

#undef LOCTEXT_NAMESPACE
