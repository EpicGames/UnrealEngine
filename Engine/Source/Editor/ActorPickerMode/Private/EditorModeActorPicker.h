// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "EdMode.h"
#include "ActorPickerMode.h"

class UActorPickerViewportInteraction;
class FEditorViewportClient;
class FViewport;
class SWindow;
class UViewportInteractionsBehaviorSource;

namespace EPickState
{
	enum Type
	{
		NotOverViewport,
		OverViewport,
		OverIncompatibleActor,
		OverActor,
	};
}

/**
 * Editor mode used to interactively pick actors in the level editor viewports.
 */
class FEdModeActorPicker : public FEdMode
{
public:
	FEdModeActorPicker();

	/** Begin FEdMode interface */
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	virtual bool MouseEnter(FEditorViewportClient* ViewportClient, FViewport* Viewport,int32 x, int32 y) override;
	virtual bool MouseLeave(FEditorViewportClient* ViewportClient, FViewport* Viewport) override;
	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override;
	virtual bool LostFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport) override;
	virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	virtual bool GetCursor(EMouseCursor::Type& OutCursor) const override;
	virtual bool UsesToolkits() const override;
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;
	virtual void Enter() override;
	virtual void Exit() override;
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }
	/** End FEdMode interface */

	/** Delegate used to display information about picking near the cursor */
	FText GetCursorDecoratorText() const;

	void UpdateHoveredActor(const HHitProxy* HitProxy);
	void OnTrySelectActor(const HHitProxy* HitProxy);
	
	bool IsActorValid(const AActor *const Actor) const;
	

	/** Flag for display state */
	TWeakObjectPtr<AActor> HoveredActor;

	/** Flag for display state */
	EPickState::Type PickState;

	/** The window that owns the decorator widget */
	TSharedPtr<SWindow> CursorDecoratorWindow;

	/** Delegates used to pick actors */
	FOnActorSelected OnActorSelected;
	FOnGetAllowedClasses OnGetAllowedClasses;
	FOnShouldFilterActor OnShouldFilterActor;

private:

	void OnBuildViewportInteractions(UViewportInteractionsBehaviorSource* Source);
	TWeakObjectPtr<UActorPickerViewportInteraction> ViewportInteraction;

	/** Enable/Disable viewport widget. */

	enum class WidgetVisibilityState : uint8
	{
		StoreAndHide,
		Restore
	};

	void UpdateWidgetVisibility(const WidgetVisibilityState InState, FEditorViewportClient* InViewportClient = nullptr);
	TFunction<void()> WidgetVisibilityFunction = TFunction<void()>();
};
