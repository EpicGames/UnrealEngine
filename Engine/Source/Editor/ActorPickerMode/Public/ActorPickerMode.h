// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#define UE_API ACTORPICKERMODE_API

class AActor;
class FEditorModeTools;

DECLARE_DELEGATE_OneParam( FOnGetAllowedClasses, TArray<const UClass*>& );
DECLARE_DELEGATE_OneParam( FOnActorSelected, AActor* );
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnShouldFilterActor, const AActor*);

/**
 * Actor picker mode module
 */
class FActorPickerModeModule : public IModuleInterface
{
public:
	// IModuleInterface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
	// End of IModuleInterface

	/** 
	 * Enter actor picking mode (note: will cancel any current actor picking)
	 * @param	InOnGetAllowedClasses	Delegate used to only allow actors using a particular set of classes (empty to accept all actor classes; works alongside InOnShouldFilterActor)
	 * @param	InOnShouldFilterActor	Delegate used to only allow particular actors (empty to accept all actors; works alongside InOnGetAllowedClasses)
	 * @param	InOnActorSelected		Delegate to call when a valid actor is selected
	 */
	UE_API void BeginActorPickingMode(FOnGetAllowedClasses InOnGetAllowedClasses, FOnShouldFilterActor InOnShouldFilterActor, FOnActorSelected InOnActorSelected) const;

	/** Exit actor picking mode */
	UE_API void EndActorPickingMode() const;

	/** @return Whether or not actor picking mode is currently active */
	UE_API bool IsInActorPickingMode() const;
	
private:

	/** Handler for when the application is deactivated. */
	UE_API void OnApplicationDeactivated(const bool IsActive) const;

	static UE_API FEditorModeTools* GetLevelEditorModeManager();

private:
	FDelegateHandle OnApplicationDeactivatedHandle;
};

#undef UE_API
